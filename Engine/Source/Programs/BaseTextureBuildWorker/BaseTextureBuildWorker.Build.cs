// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BaseTextureBuildWorker : TextureBuildWorker
{
	public BaseTextureBuildWorker(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"Engine",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"TextureBuild",
			"TextureBuildUtilities",
			"TextureFormat",
			"TextureFormatASTC",
			"TextureFormatETC2",
			"TextureFormatUncompressed",
			"TextureFormatOodle",
		});
	}
}
