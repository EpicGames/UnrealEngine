// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class BaseTextureBuildWorkerTarget : TextureBuildWorkerTarget
{
	public BaseTextureBuildWorkerTarget(TargetInfo Target) : base(Target)
	{
		LaunchModuleName = "BaseTextureBuildWorker";
	}
}
